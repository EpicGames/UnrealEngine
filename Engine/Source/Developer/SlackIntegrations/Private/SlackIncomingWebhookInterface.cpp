// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlackIncomingWebhookInterface.h"
#include "HttpModule.h"
#include "SlackIntegrations.h"

bool FSlackIncomingWebhookInterface::SendMessage(const FSlackIncomingWebhook& InWebhook, const FSlackMessage& InMessage) const
{
	auto HttpRequest = CreateHttpRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetURL(InWebhook.WebhookUrl);

	TArray<uint8> Payload;
	GetPayload(InWebhook, InMessage, Payload);
	HttpRequest->SetContent(Payload);

	return HttpRequest->ProcessRequest();
}

TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FSlackIncomingWebhookInterface::CreateHttpRequest() const
{
	auto Request = FHttpModule::Get().CreateRequest();
	Request->OnProcessRequestComplete().BindRaw(this, &FSlackIncomingWebhookInterface::OnProcessRequestComplete);
	return Request;
}

void FSlackIncomingWebhookInterface::OnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) const
{
}

void FSlackIncomingWebhookInterface::GetPayload(const FSlackIncomingWebhook& InWebhook, const FSlackMessage& InMessage, TArray<uint8>& OutPayload)
{
	FString Payload;
	Payload += TEXT("{");

	if (!InWebhook.Channel.IsEmpty())
	{
		Payload += FString::Printf(TEXT("\"channel\": \"%s\", "), *JsonEncode(InWebhook.Channel));
	}
	if (!InWebhook.Username.IsEmpty())
	{
		Payload += FString::Printf(TEXT("\"username\": \"%s\", "), *JsonEncode(InWebhook.Username));
	}
	if (!InWebhook.IconEmoji.IsEmpty())
	{
		Payload += FString::Printf(TEXT("\"icon_emoji\": \"%s\", "), *JsonEncode(InWebhook.IconEmoji));
	}
	Payload += FString::Printf(TEXT("\"text\": \"%s\"}"), *JsonEncode(InMessage.MessageText));

	FTCHARToUTF8 Converter((const TCHAR*)*Payload, Payload.Len());
	const int32 Length = Converter.Length();
	OutPayload.Reset(Length);
	OutPayload.AddUninitialized(Length);
	CopyAssignItems((ANSICHAR*)OutPayload.GetData(), Converter.Get(), Length);
}

FString FSlackIncomingWebhookInterface::JsonEncode(const FString& InString)
{
	FString OutString = InString;
	OutString.ReplaceInline(TEXT("\\"), TEXT("\\\\"));
	OutString.ReplaceInline(TEXT("\r\n"), TEXT("\\n"));
	OutString.ReplaceInline(TEXT("\r"), TEXT("\\n"));
	OutString.ReplaceInline(TEXT("\n"), TEXT("\\n"));
	OutString.ReplaceInline(TEXT("\""), TEXT("\\\""));
	return OutString;
}
