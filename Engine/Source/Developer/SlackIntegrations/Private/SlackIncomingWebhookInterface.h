// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IHttpRequest.h"
#include "ISlackIncomingWebhookInterface.h"

class IHttpRequest;
struct FSlackIncomingWebhook;
struct FSlackMessage;

/**
 * Concrete implementation of ISlackIncomingWebhookInterface in module's private code.
 * Sends Incoming Webhook messages to Slack.
 */
class FSlackIncomingWebhookInterface : public ISlackIncomingWebhookInterface
{
public:
	/** Empty virtual destructor because class has virtual methods */
	virtual ~FSlackIncomingWebhookInterface() {}

	/**
	 * Send an incoming webhook message to Slack
	 * @param InWebhook		Webhook parameters sets where and how to send the message
	 * @param InMessage		The message to send to Slack
	 * @return				True if the http request was send successfully, otherwise false.
	 */
	virtual bool SendMessage(const FSlackIncomingWebhook& InWebhook, const FSlackMessage& InMessage) const override;

private:
	/** Helper used to make http requests by SendMessage() */
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateHttpRequest() const;

	/**
	 * Callback from HTTP library when a request has completed
	 * @param HttpRequest The request object
	 * @param HttpResponse The response from the server
	 * @param bSucceeded Whether a response was successfully received
	 */
	void OnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) const;

	/** Builds the json payload bytes to send in the http request to Slack */
	static void GetPayload(const FSlackIncomingWebhook& InWebhook, const FSlackMessage& InMessage, TArray<uint8>& OutPayload);

	/** Helper that escapes characters in the payload to make valid json */
	static FString JsonEncode(const FString& InString);
};
