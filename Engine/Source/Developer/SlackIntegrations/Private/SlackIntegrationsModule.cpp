// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlackIntegrationsModule.h"
#include "SlackIncomingWebhookInterface.h"

/**
 * Concrete implementation of ISlackIntegrationsModule in module's private code
 */
class FSlackIntegrationsModule : public ISlackIntegrationsModule
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override
	{
		IncomingWebhookInterface = new FSlackIncomingWebhookInterface();
	}

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override
	{
		if (IncomingWebhookInterface != nullptr)
		{
			delete IncomingWebhookInterface;
			IncomingWebhookInterface = nullptr;
		}
	}

	/** Get the incoming webhook interface for sending messages to Slack */
	virtual ISlackIncomingWebhookInterface& GetIncomingWebhookInterface()
	{
		check(IncomingWebhookInterface != nullptr);
		return *IncomingWebhookInterface;
	}

private:
	/** Singleton interface sending incoming webhook messages to Slack */
	FSlackIncomingWebhookInterface* IncomingWebhookInterface;
};

IMPLEMENT_MODULE(FSlackIntegrationsModule, SlackIntegrations);

