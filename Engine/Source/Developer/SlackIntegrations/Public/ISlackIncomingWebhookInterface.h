// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSlackIncomingWebhook;
struct FSlackMessage;

/**
 * Public interface for accessing Slack incoming webhook methods
 */
class ISlackIncomingWebhookInterface
{
public:
	virtual bool SendMessage(const FSlackIncomingWebhook& InWebhook, const FSlackMessage& InMessage) const = 0;
};
