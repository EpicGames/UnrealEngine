// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"

/**  
 * Simple struct represents the parameters used for incoming webhook requests
 */
struct FSlackIncomingWebhook
{
	/** Full URL from your Incoming Webhook Integrations on the Slack website */
	FString WebhookUrl;

	/** Overrides the channel to which the message is posted. The Incoming Webhook Integrations created on the Slack website will have a default channel. */
	FString Channel;

	/** Overrides the username with which the message is posted. The Incoming Webhook Integrations created on the Slack website will have a default username. */
	FString Username;

	/** Overrides the emoji for the username with which the message is posted. The Incoming Webhook Integrations created on the Slack website will have a default emoji. */
	FString IconEmoji;
};

/**
 * Simple struct represents the parameters of a Slack message
 */
struct FSlackMessage
{
	/** Message content */
	FString MessageText;
};
