// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlackIntegrations : ModuleRules
{
    public SlackIntegrations(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
            new string[] {
				"HTTP",
			}
		);
    }
}

